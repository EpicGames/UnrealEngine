{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Cinematic Prestreaming",
	"Description": "Adds a way to record certain types of streaming data requests in cinematic cutscenes. The requests can then be played back in advance on the Sequencer timeline to pre-stream data during normal gameplay/rendering.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "CinematicPrestreaming",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CinematicPrestreamingEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins":
	[
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		}
	]
}