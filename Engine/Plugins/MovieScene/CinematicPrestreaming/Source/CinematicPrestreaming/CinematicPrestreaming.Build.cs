// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CinematicPrestreaming : ModuleRules
{
	public CinematicPrestreaming(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "CinePrestream";

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MovieScene"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"RenderCore",
				"RHI"
			}
		);
	}
}
