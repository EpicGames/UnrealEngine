// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingData.h"

#include "CinePrestreamingLog.h"
#include "EngineModule.h"
#include "Misc/FrameNumber.h"
#include "RendererInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CinePrestreamingData)

void UCinePrestreamingData::PostLoad()
{
	Super::PostLoad();

	bool bReportDeprecatedData = false;
	for (FCinePrestreamingVTData& Data : VirtualTextureDatas)
	{
		if (Data.PageIds_DEPRECATED.Num() != 0 && Data.RequestData.Num() == 0)
		{
			bReportDeprecatedData = true;
#if WITH_EDITOR
			// Transcribe from old format to new.
			TMap<uint64, uint32> CopiedRequests;
			for (uint64 PageId : Data.PageIds_DEPRECATED)
			{
				const uint64 ModifiedPageId = (PageId & 0xffffffff00000000) | ((PageId & 0xf0000000) >> 4) | ((PageId & 0x0f000000) << 4) | (PageId & 0x00ffffff);
				const uint32 AssumedPriority = 0xffff;
				CopiedRequests.FindOrAdd(ModifiedPageId) = AssumedPriority;
			}
			// Pack to RequestData stream.
			GetRendererModule().PackVirtualTextureRequestsToStream(CopiedRequests, Data.RequestData);
#endif
		}
	}
	if (bReportDeprecatedData)
	{
#if WITH_EDITOR
		UE_LOGF(LogCinePrestreaming, Log, "CinePrestreamingData '%ls' data is being updated on PostLoad. Re-record to fix.", *GetName());
#else
		UE_LOGF(LogCinePrestreaming, Log, "CinePrestreamingData '%ls' data is not available in cooked build. Re-record to fix.", *GetName());
#endif
	}
}

void UCinePrestreamingData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	int32 Size = sizeof(UCinePrestreamingData);
	Size += Times.GetAllocatedSize();

	Size += VirtualTextureDatas.GetAllocatedSize();
	for (FCinePrestreamingVTData const& Data : VirtualTextureDatas)
	{
		Size += Data.RequestData.GetAllocatedSize();
	}

	Size += NaniteDatas.GetAllocatedSize();
	for (FCinePrestreamingNaniteData const& Data : NaniteDatas)
	{
		Size += Data.RequestData.GetAllocatedSize();
	}
	
	static FName TagName(TEXT("CinePrestreaming Data"));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(TagName, Size);
}
