// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingLog.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, CinematicPrestreaming);

DEFINE_LOG_CATEGORY(LogCinePrestreaming);
