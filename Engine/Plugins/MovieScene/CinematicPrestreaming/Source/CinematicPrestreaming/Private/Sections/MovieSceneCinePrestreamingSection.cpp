// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCinePrestreamingSection.h"

#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "TrackInstances/MovieSceneCinePrestreamingTrackInstance.h"

UMovieSceneCinePrestreamingSection::UMovieSceneCinePrestreamingSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::RestoreState);
}

void UMovieSceneCinePrestreamingSection::Serialize(FArchive& Ar)
{
#if WITH_EDITOR	
	// If we want async load behavior then cook the soft instead of the hard ref.
	if (Ar.IsCooking() && Ar.IsSaving() && bUseAsyncLoading)
	{
		TObjectPtr<UCinePrestreamingData> PrestreamingAssetBackup = PrestreamingAsset;
		// Note that we always expect SoftPrestreamingAsset to be null in editor, but we do backup/restore here for completeness.
		TSoftObjectPtr<UCinePrestreamingData> SoftPrestreamingAssetBackup = SoftPrestreamingAsset;
		
		SoftPrestreamingAsset = PrestreamingAsset;
		PrestreamingAsset = nullptr;

		Super::Serialize(Ar);

		PrestreamingAsset = PrestreamingAssetBackup;
		SoftPrestreamingAsset = SoftPrestreamingAssetBackup;
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

void UMovieSceneCinePrestreamingSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FMovieSceneTrackInstanceComponent TrackInstance{ decltype(FMovieSceneTrackInstanceComponent::Owner)(this), UMovieSceneCinePrestreamingTrackInstance::StaticClass() };

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(FBuiltInComponentTypes::Get()->Tags.Root)
		.Add(FBuiltInComponentTypes::Get()->TrackInstance, TrackInstance)
	);
}

void UMovieSceneCinePrestreamingSection::AsyncStreamingAddRef()
{
	if (!bUseAsyncLoading || SoftPrestreamingAsset.IsNull())
	{
		return;
	}
	
	if (AsyncLoadingRefCount++ > 0)
	{
		return;
	}

	FStreamableManager& StreamableManager = UAssetManager::GetStreamableManager();
	TWeakObjectPtr<UMovieSceneCinePrestreamingSection> WeakThis = this;
	AsyncLoadHandle = StreamableManager.RequestAsyncLoad(SoftPrestreamingAsset.ToSoftObjectPath(), [WeakThis]()
	{
		if (UMovieSceneCinePrestreamingSection* HardThis = WeakThis.Get())
		{
			HardThis->PrestreamingAsset = HardThis->SoftPrestreamingAsset.Get();
		}
	});
}

void UMovieSceneCinePrestreamingSection::AsyncStreamingReleaseRef()
{
	if (!bUseAsyncLoading || SoftPrestreamingAsset.IsNull())
	{
		return;
	}
	
	if (AsyncLoadingRefCount == 0 || --AsyncLoadingRefCount > 0)
	{
		return;
	}

	if (AsyncLoadHandle.IsValid())
	{
		AsyncLoadHandle->CancelHandle();
		AsyncLoadHandle.Reset();
	}
	PrestreamingAsset = nullptr;
}
