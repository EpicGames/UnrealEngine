// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstances/MovieSceneCinePrestreamingTrackInstance.h"

#include "CinePrestreamingData.h"
#include "EngineModule.h"
#include "MovieScene.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "Sections/MovieSceneCinePrestreamingSection.h"

DECLARE_CYCLE_STAT(TEXT("Cinematic PrestreamingTrack Animate"), MovieSceneEval_CinePrestreamingTrack_Animate, STATGROUP_MovieSceneEval);

static TAutoConsoleVariable<int32> CVarCinematicPreStreamQualityLevel(
	TEXT("MovieScene.PreStream.QualityLevel"),
	2,
	TEXT("We discard prestreaming sections with QualityLevel greater than this."),
	ECVF_ReadOnly
);

static bool SupportSectionQualityLevel(UMovieSceneCinePrestreamingSection const* InSection)
{
	return CVarCinematicPreStreamQualityLevel.GetValueOnGameThread() >= InSection->GetQualityLevel();
}

void UMovieSceneCinePrestreamingTrackInstance::OnInputAdded(const FMovieSceneTrackInstanceInput& InInput)
{
	if (UMovieSceneCinePrestreamingSection* Section = Cast<UMovieSceneCinePrestreamingSection>(InInput.Section))
	{
		if (SupportSectionQualityLevel(Section))
		{
			// Trigger any required async load for the prestreaming data.
			Section->AsyncStreamingAddRef();
		}
	}
}

void UMovieSceneCinePrestreamingTrackInstance::OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput)
{
	if (UMovieSceneCinePrestreamingSection* Section = Cast<UMovieSceneCinePrestreamingSection>(InInput.Section))
	{
		if (SupportSectionQualityLevel(Section))
		{
			// Release any async load for the prestreaming data.
			Section->AsyncStreamingReleaseRef();
		}
	}
}

void UMovieSceneCinePrestreamingTrackInstance::OnAnimate()
{
	SCOPE_CYCLE_COUNTER(MovieSceneEval_CinePrestreamingTrack_Animate);

	using namespace UE::MovieScene;

	const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();

	// Gather active prestreaming sections.
	FCinePrestreamingVTData AccumulatedVirtualTextureData;
	FCinePrestreamingNaniteData AccumulatedNaniteData;
	FIntPoint RecordedResolution = FIntPoint::ZeroValue;

	for (const FMovieSceneTrackInstanceInput& Input : GetInputs())
	{
		UMovieSceneCinePrestreamingSection* Section = Cast<UMovieSceneCinePrestreamingSection>(Input.Section);
		if (Section == nullptr)
		{
			continue;
		}

		// Early out if this input is stripped by QualityLevel setting.
		if (!SupportSectionQualityLevel(Section))
		{
			continue;
		}

		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Input.InstanceHandle);
		const FMovieSceneContext& Context = SequenceInstance.GetContext();

		// Get start/current/end time.
		const TRange<FFrameNumber> SectionRange(Input.Section->GetTrueRange());
		const FFrameNumber LocalStartTime = SectionRange.HasLowerBound() ? SectionRange.GetLowerBoundValue() : FFrameNumber(-MAX_int32);
		const FFrameNumber LocalEndTime = SectionRange.HasUpperBound() ? SectionRange.GetUpperBoundValue() : FFrameNumber(MAX_int32);
		
		// Get the time relative to the start of the section.
		const FFrameTime LocalContextTime = Context.GetTime();
		const FFrameNumber LocalEvalFrame = LocalContextTime.FloorToFrame().Value - LocalStartTime;

		// Apply an offset to start evaluation early (extends into PreRoll so must be less than number of PreRoll frames).
		const int32 OffsetTime = Input.Section->GetOffsetTime().Get(0).GetFrame().Value;
		const FFrameRate DisplayRate = Input.Section->GetTypedOuter<UMovieScene>()->GetDisplayRate();
		const int32 OffsetTimeFrames = FFrameRate::TransformTime(FFrameTime(OffsetTime), DisplayRate, Context.GetFrameRate()).FloorToFrame().Value;
		const int32 PreRollFrames = Input.Section->GetPreRollFrames();
		const FFrameNumber StartOffsetFrames = FMath::Clamp(OffsetTimeFrames, 0, PreRollFrames);

		// Early out if we're out of evaluation range.
		// This can be in the PreRoll, or in the dead space at the end of the shot caused by the OffsetTime.
		if ((LocalEvalFrame + StartOffsetFrames).Value < 0 || (LocalEvalFrame + StartOffsetFrames) > (LocalEndTime - LocalStartTime))
		{
			continue;
		}

		UCinePrestreamingData* PrestreamingData = Section->GetPrestreamingAsset();
		if (PrestreamingData == nullptr)
		{
			// We can get here if there is no prestreaming data asset set or if async loading has not had time to complete.
			continue;
		}

		// Early out if there is no prestreaming data content, or if data is invalid.
		const int32 ItemCount = PrestreamingData->Times.Num();
		if (ItemCount == 0 || ItemCount != PrestreamingData->VirtualTextureDatas.Num() || ItemCount != PrestreamingData->NaniteDatas.Num())
		{
			continue;
		}

		// Deal with offset due to internal start time of recorded data.
		const FFrameNumber DataOffsetFrames = PrestreamingData->Times[0];

		// Find the closest (floored) keyframe time.
		const int32 Index = FMath::Max(0, Algo::UpperBound(PrestreamingData->Times, (LocalEvalFrame + StartOffsetFrames + DataOffsetFrames).Value) - 1);

		// Walk back to last non-empty set of data, which we repeatedly request until the next set of non-empty data. 
		int32 VirtualTextureDataIndex = Index;
		while (VirtualTextureDataIndex > 0 && PrestreamingData->VirtualTextureDatas[VirtualTextureDataIndex].RequestData.Num() == 0)
		{
			VirtualTextureDataIndex--;
		}
		AccumulatedVirtualTextureData.RequestData.Append(PrestreamingData->VirtualTextureDatas[VirtualTextureDataIndex].RequestData);

		int32 NaniteIndex = Index;
		while (NaniteIndex > 0 && PrestreamingData->NaniteDatas[NaniteIndex].RequestData.Num() == 0)
		{
			NaniteIndex--;
		}
		AccumulatedNaniteData.RequestData.Append(PrestreamingData->NaniteDatas[NaniteIndex].RequestData);

		// Should be safe to assume that if we have multiple tracks then they were recorded at the same resolution.
		ensure(RecordedResolution == FIntPoint::ZeroValue || RecordedResolution == PrestreamingData->RecordedResolution);
		RecordedResolution = PrestreamingData->RecordedResolution;
	}

	// Submit any prestreaming data to the renderer.
	if (AccumulatedVirtualTextureData.RequestData.Num() > 0 || AccumulatedNaniteData.RequestData.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(RequestData)([RecordedResolution, VirtualTextureRequestData = MoveTemp(AccumulatedVirtualTextureData.RequestData), NaniteRequestData = MoveTemp(AccumulatedNaniteData.RequestData)](FRHICommandListImmediate& RHICmdList) mutable
		{
			if (VirtualTextureRequestData.Num())
			{
				GetRendererModule().RequestVirtualTextureTiles(MoveTemp(VirtualTextureRequestData), RecordedResolution);
			}
			if (NaniteRequestData.Num())
			{
				GetRendererModule().RequestNanitePages(MoveTemp(NaniteRequestData));
			}
		});
	}
}
