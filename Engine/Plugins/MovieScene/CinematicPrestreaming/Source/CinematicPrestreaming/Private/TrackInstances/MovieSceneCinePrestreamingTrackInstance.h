// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "MovieSceneCinePrestreamingTrackInstance.generated.h"

UCLASS()
class UMovieSceneCinePrestreamingTrackInstance : public UMovieSceneTrackInstance
{
	GENERATED_BODY()

protected:
	/** UMovieSceneTrackInstance interface */
	void OnInputAdded(const FMovieSceneTrackInstanceInput& InInput) override;
	void OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput) override;
	void OnAnimate() override;
};
