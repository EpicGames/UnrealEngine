// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCinePrestreamingTrack.h"
#include "Sections/MovieSceneCinePrestreamingSection.h"

#define LOCTEXT_NAMESPACE "MovieSceneCinePrestreamingTrack"

UMovieSceneCinePrestreamingTrack::UMovieSceneCinePrestreamingTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.bEvaluateInPreroll = EvalOptions.bEvaluateInPostroll = true;
}

bool UMovieSceneCinePrestreamingTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneCinePrestreamingTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

void UMovieSceneCinePrestreamingTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneCinePrestreamingTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

bool UMovieSceneCinePrestreamingTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCinePrestreamingSection::StaticClass();
}

UMovieSceneSection* UMovieSceneCinePrestreamingTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCinePrestreamingSection>(this, NAME_None, RF_Transactional);
}

const TArray<UMovieSceneSection*>& UMovieSceneCinePrestreamingTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneCinePrestreamingTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneCinePrestreamingTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("DisplayName", "Cinematic Prestreaming");
}
#endif

#undef LOCTEXT_NAMESPACE
