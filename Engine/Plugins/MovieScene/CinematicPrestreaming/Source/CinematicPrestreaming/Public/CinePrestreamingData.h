// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/DateTime.h"
#include "UObject/Object.h"
#include "CinePrestreamingData.generated.h"

USTRUCT()
struct FCinePrestreamingVTData
{
	GENERATED_BODY()
	
	/**
	 * Deprecated container format. 
	 * Will be fixed up into RequestData during editor PostLoad.
	 * But it is recommended to re-record the shot to benefit from a more efficient asset size.
	 */
	UPROPERTY()
	TArray<uint64> PageIds_DEPRECATED;

	/** Compacted request stream for passing virtual texture requests directly to the renderer. */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	TArray<uint32> RequestData;
};

USTRUCT()
struct FCinePrestreamingNaniteData
{
	GENERATED_BODY()
	
	/** Compacted request stream for passing nanite requests directly to the renderer. */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	TArray<uint32> RequestData;
};

UCLASS(BlueprintType)
class CINEMATICPRESTREAMING_API UCinePrestreamingData : public UObject
{
	GENERATED_BODY()

	// UObject Interface
	virtual void PostLoad() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	// ~UObject Interface

public:
	/** Array of frame number times for the corresponding entries in the VirtualTextureDatas and NaniteDatas arrays. */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	TArray<FFrameNumber> Times;
	
	/** 
	 * Array of virtual texture request data. 
	 * Can have empty array slots because the recording process groups requests for time periods to reduce memory cost of repeated data.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	TArray<FCinePrestreamingVTData> VirtualTextureDatas;

	/**
	 * Array of nanite request data.
	 * Can have empty array slots because the recording process groups requests for time periods to reduce memory cost of repeated data.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	TArray<FCinePrestreamingNaniteData> NaniteDatas;

	/** Time that this asset was generated (in UTC). Used to give better context about how up to date an asset is as they are hard to preview. */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	FDateTime RecordedTime;

	/** What resolution was this asset generated at? Recordings are resolution dependent as different mips will be chosen for different resolutions. */
	UPROPERTY(VisibleAnywhere, Category = "Cinematic Prestreaming")
	FIntPoint RecordedResolution;
};
