// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "CinePrestreamingData.h"
#include "MovieSceneCinePrestreamingSection.generated.h"

class UCinePrestreamingData;
struct FStreamableHandle;

/** Movie Scene Section representing a Prestreaming asset. */
UCLASS(MinimalAPI)
class UMovieSceneCinePrestreamingSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:
	UMovieSceneCinePrestreamingSection(const FObjectInitializer& ObjInit);

	/** Get the prestreaming asset pointer. This may return nullptr in cooked build if we are using async loading. */
	UFUNCTION(BlueprintPure, Category = "Cinematic Prestreaming")
	UCinePrestreamingData* GetPrestreamingAsset() const { return PrestreamingAsset; }

	UFUNCTION(BlueprintCallable, Category = "Cinematic Prestreaming")
	void SetPrestreamingAsset(UCinePrestreamingData* InData) { PrestreamingAsset = InData; }

	/** If MovieScene.PreStream.QualityLevel is less than this then discard this section at runtime. */
	UFUNCTION(BlueprintPure, Category = "Cinematic Prestreaming")
	int32 GetQualityLevel() const { return QualityLevel; };

	UFUNCTION(BlueprintCallable, Category = "Cinematic Prestreaming")
	void SetQualityLevel(const int32 InLevel) { QualityLevel = InLevel; }

	UFUNCTION(BlueprintCallable, Category = "Cinematic Prestreaming")
	void SetStartFrameOffset(const int32 InOffset) { StartFrameOffset = InOffset; }

	/** Add a reference to the async loader. When the ref count is non-zero the prestreaming asset should be loaded or loading. */
	void AsyncStreamingAddRef();

	/** Release the reference to the async loader. When the ref count is zero the prestreaming asset can be unloaded. */
	void AsyncStreamingReleaseRef();

	/** UMovieSceneSection interface */
	virtual TOptional<FFrameTime> GetOffsetTime() const override { return TOptional<FFrameTime>(StartFrameOffset); }

protected:
	/** UObject interface */
	virtual void Serialize(FArchive& Ar) override;

	/** IMovieSceneEntityProvider interface */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

protected:
	/** The asset containing cinematic prestreaming data to use for this section. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Prestreaming)
	TObjectPtr<UCinePrestreamingData> PrestreamingAsset;

	/** Soft ref version of the asset for use by async loading. */
	UPROPERTY()
	TSoftObjectPtr<UCinePrestreamingData> SoftPrestreamingAsset;

	/** Number of frames by which to offset the evaluation. Larger values cause prestreaming to happen earlier. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Prestreaming)
	int32 StartFrameOffset = 0;

	/** If MovieScene.PreStream.QualityLevel is less than this then discard this section at runtime. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Prestreaming, AdvancedDisplay)
	int32 QualityLevel = 0;

	/** 
	 * Set true to enable async loading of the prestreaming data in game.
	 * This can reduce memory overhead for a long cinematic with multiple prestreaming shots.
	 * Async loading requires sufficient preroll time to load the asset and still have enough time to run StartFrameOffset frames ahead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Prestreaming, AdvancedDisplay)
	bool bUseAsyncLoading= false;

	/** Ref count for async loading of PrestreamingAsset. */
	uint32 AsyncLoadingRefCount = 0;

	/** Handle for async loading of PrestreamingAsset. */
	TSharedPtr<FStreamableHandle> AsyncLoadHandle;
};
