// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneCinePrestreamingTrack.generated.h"

/**
 * A track that controls playback of streaming triggers for some rendering systems.
 * This is to ensure that data is available before use where by default for systems like
 * virtual texture and nanite streaming is driven by what is already visible on screen.
 */
UCLASS( MinimalAPI )
class UMovieSceneCinePrestreamingTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:
	UMovieSceneCinePrestreamingTrack(const FObjectInitializer& ObjectInitializer);

	/** UMovieSceneTrack interface */
	bool IsEmpty() const override;
	void AddSection(UMovieSceneSection& Section) override;
	void RemoveSection(UMovieSceneSection& Section) override;
	void RemoveSectionAt(int32 SectionIndex) override;
	bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	UMovieSceneSection* CreateNewSection() override;
	const TArray<UMovieSceneSection*>& GetAllSections() const override;
	bool HasSection(const UMovieSceneSection& Section) const override;
	bool SupportsMultipleRows() const override { return true; }

#if WITH_EDITORONLY_DATA
	FText GetDefaultDisplayName() const override;
#endif

private:
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
