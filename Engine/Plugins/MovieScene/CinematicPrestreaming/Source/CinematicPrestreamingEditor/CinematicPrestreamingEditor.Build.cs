// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CinematicPrestreamingEditor : ModuleRules
{
	public CinematicPrestreamingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "CinePrestreamEd";

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CinematicPrestreaming",
				"LevelSequence",
				"MovieRenderPipelineCore",
				"MovieRenderPipelineRenderPasses",
				"MovieRenderPipelineEditor",
				"MovieScene",
				"SequencerCore",
				"Sequencer",
				"UnrealEd",
				"EditorSubsystem"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"RenderCore",
				"Slate",
				"SlateCore",
			}
		);
	}
}
