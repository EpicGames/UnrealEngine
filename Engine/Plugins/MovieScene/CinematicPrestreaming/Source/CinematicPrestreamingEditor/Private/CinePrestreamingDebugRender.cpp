// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingDebugRender.h"

#include "VT/VirtualTextureVisualizationData.h"

UCinePrestreamingDebugRender::UCinePrestreamingDebugRender() 
	: UMoviePipelineDeferredPassBase()
	, VirtualTextureDebugMode(EVirtualTextureVisualizationMode::PendingMips)
	, PreviousMode(EVirtualTextureVisualizationMode::None)
{
	PassIdentifier = FMoviePipelinePassIdentifier("VirtualTexturePendingMips");
	AdditionalPostProcessMaterials.Reset();
}

#if WITH_EDITOR
FText UCinePrestreamingDebugRender::GetDisplayText() const
{
	return NSLOCTEXT("MovieRenderPipeline", "DeferredBasePassSetting_DisplayName_VTPendingMips", "Deferred Rendering (VT Debug)");
}
#endif

void UCinePrestreamingDebugRender::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	Super::SetupImpl(InPassInitSettings);

	PreviousMode = GetVirtualTextureVisualizationData().SetModeUsingConsoleCommand(VirtualTextureDebugMode);
}

void UCinePrestreamingDebugRender::TeardownImpl()
{
	Super::TeardownImpl();

	GetVirtualTextureVisualizationData().SetModeUsingConsoleCommand(PreviousMode);
}

void UCinePrestreamingDebugRender::GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const
{
	const bool bVisualizeVirtualTexture = VirtualTextureDebugMode != EVirtualTextureVisualizationMode::None;

	OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
	OutShowFlag.SetVisualizeVirtualTexture(bVisualizeVirtualTexture);
	OutViewModeIndex = bVisualizeVirtualTexture ? EViewModeIndex::VMI_VisualizeVirtualTexture : EViewModeIndex::VMI_Lit;
}
