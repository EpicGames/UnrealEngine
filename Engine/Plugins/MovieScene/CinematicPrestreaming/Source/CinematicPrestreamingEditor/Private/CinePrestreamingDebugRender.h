// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineDeferredPasses.h"
#include "CinePrestreamingDebugRender.generated.h"

enum class EVirtualTextureVisualizationMode : uint8;

UCLASS(BlueprintType)
class UCinePrestreamingDebugRender : public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

public:
	UCinePrestreamingDebugRender();

	virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	virtual void TeardownImpl() override;
#if WITH_EDITOR
	virtual FText GetDisplayText() const override;
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override;
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }

 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
 	EVirtualTextureVisualizationMode VirtualTextureDebugMode;

private:
	EVirtualTextureVisualizationMode PreviousMode;
};
