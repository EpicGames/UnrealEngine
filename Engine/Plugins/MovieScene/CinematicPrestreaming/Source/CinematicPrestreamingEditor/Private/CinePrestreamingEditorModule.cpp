// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingEditorModule.h"

#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "TrackEditors/CinePrestreamingTrackEditor.h"

#define LOCTEXT_NAMESPACE "FCinePrestreamingEditorModule"

void FCinePrestreamingEditorModule::StartupModule()
{
	RegisterTrackEditors();
}

void FCinePrestreamingEditorModule::ShutdownModule()
{
	UnregisterTrackEditors();
}

void FCinePrestreamingEditorModule::RegisterTrackEditors()
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	CinePrestreamingCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FCinePrestreamingTrackEditor::CreateTrackEditor));
}

void FCinePrestreamingEditorModule::UnregisterTrackEditors()
{
	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModule)
	{
		SequencerModule->UnRegisterTrackEditor(CinePrestreamingCreateEditorHandle);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCinePrestreamingEditorModule, CinematicPrestreamingEditor)