// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingEditorSubsystem.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "CinePrestreamingData.h"
#include "CinePrestreamingRecorderSetting.h"
#include "FileHelpers.h"
#include "Misc/PackageName.h"
#include "ObjectTools.h"

void UCinePrestreamingEditorSubsystem::CreatePackagesFromGeneratedData(TArray<FMoviePipelineCinePrestreamingGeneratedData>& InOutData)
{
	for (FMoviePipelineCinePrestreamingGeneratedData& Data : InOutData)
	{
		const FString FixedAssetName = ObjectTools::SanitizeObjectName(Data.AssetName);
		FString NewPackageName = FPackageName::GetLongPackagePath(Data.PackagePath) + TEXT("/") + FixedAssetName;

		NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);

		FText OutErrorReason;
		if (!FPackageName::IsValidLongPackageName(NewPackageName, false, &OutErrorReason))
		{
			continue;
		}

		if (UPackage* OldPackage = FindObject<UPackage>(nullptr, *NewPackageName))
		{
			FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), UPackage::StaticClass(), "DEAD_CinePrestreamingSettingAsset");
			OldPackage->Rename(*UniqueName.ToString());
			OldPackage->SetFlags(RF_Transient);
		}

		UPackage* NewPackage = CreatePackage(*NewPackageName);

		// Try to fully load the package (if it already exists) so we can save over it.
		LoadPackage(NewPackage, *NewPackageName, LOAD_None);

		// Duplicate the data asset into this package
		UCinePrestreamingData* NewPrestreamingData = Cast<UCinePrestreamingData>(StaticDuplicateObject(Data.StreamingData, NewPackage, FName(*FixedAssetName), RF_NoFlags));
		NewPrestreamingData->SetFlags(RF_Public | RF_Transactional | RF_Standalone);
		NewPrestreamingData->MarkPackageDirty();

		Data.StreamingData = NewPrestreamingData;

		// Mark it so it shows up in the Content Browser immediately
		FAssetRegistryModule::AssetCreated(NewPrestreamingData);

		// If they want to save, ask them to save (and add to version control)
		TArray<UPackage*> Packages;
		Packages.Add(NewPrestreamingData->GetOutermost());

		UEditorLoadingAndSavingUtils::SavePackages(Packages, true);
	}
}
