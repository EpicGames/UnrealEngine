// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingGraphNode.h"

#include "CinePrestreamingData.h"
#include "CinePrestreamingEditorSubsystem.h"
#include "EngineModule.h"
#include "Kismet/KismetSystemLibrary.h"
#include "LevelSequence.h"
#include "MoviePipelineBase.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineDataTypes.h"
#include "MovieScene.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "Sections/MovieSceneCinePrestreamingSection.h"
#include "Tracks/MovieSceneCinePrestreamingTrack.h"

#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphFilenameResolveParams.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(CinePrestreamingGraphNode)

namespace CinePrestreamingGraphNode
{
	/** Show flags to disable when bDisableAdvanceRenderFeatures is enabled. */
	static const TArray<FString>& GetShowFlagsToDisable()
	{
		static const TArray<FString> Flags =
		{
			TEXT("ShowFlag.PostProcessing"),
			TEXT("ShowFlag.Lighting"),
			TEXT("ShowFlag.Atmosphere"),
			TEXT("ShowFlag.VolumetricFog"),
			TEXT("ShowFlag.VolumetricLightmap"),
			TEXT("ShowFlag.LumenGlobalIllumination"),
			TEXT("ShowFlag.LumenReflections"),
			TEXT("ShowFlag.AmbientOcclusion"),
			TEXT("ShowFlag.DistanceFieldAO"),
		};
		return Flags;
	}
}

UCinePrestreamingGraphNode::UCinePrestreamingGraphNode()
{
	PackageDirectory.Path = TEXT("/Game/Cinematics/Prestreaming/{sequence_name}/{shot_name}/");
}

#if WITH_EDITOR

FText UCinePrestreamingGraphNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieGraphNodes", "CinePrestreamingGraphNode_Title", "Prestreaming Recorder");
}

FText UCinePrestreamingGraphNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "FileOutput_Category", "Output Type");
}

FText UCinePrestreamingGraphNode::GetKeywords() const
{
	return NSLOCTEXT("MovieGraphNodes", "CinePrestreamingGraphNode_Keywords", "prestreaming virtual texture nanite streaming");
}

FLinearColor UCinePrestreamingGraphNode::GetNodeTitleColor() const
{
	// Teal to match other output nodes
	return FLinearColor(0.047f, 0.654f, 0.537f);
}

#endif // WITH_EDITOR

EMovieGraphBranchRestriction UCinePrestreamingGraphNode::GetBranchRestriction() const
{
	// One recorder per pipeline, not one per render layer
	return EMovieGraphBranchRestriction::Globals;
}

void UCinePrestreamingGraphNode::OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask)
{
	// First call: set up the recording infrastructure.
	// Note: the very first output frame's data is missed because delegates are registered here,
	// AFTER that frame has finished rendering. This is an accepted limitation of the lazy
	// registration workaround; warmup frames precede actual output frames.
	if (!ActivePipeline.IsValid())
	{
		ActivePipeline = InPipeline;
		PrevActiveShotIndex = -1;
		bShowFlagsDisabled = false;

		// Pre-allocate to avoid reallocation while the render thread is writing into SegmentData.
		SegmentData.Reset();
		SegmentData.AddDefaulted(InPipeline->GetActiveShotList().Num());

		if (bDisableAdvanceRenderFeatures)
		{
			for (const FString& Flag : CinePrestreamingGraphNode::GetShowFlagsToDisable())
			{
				UKismetSystemLibrary::ExecuteConsoleCommand(nullptr, Flag + TEXT(" 0"), nullptr);
			}
			bShowFlagsDisabled = true;
		}

		// Register per-frame delegates to record VT/Nanite requests.
		BeginFrameDelegate = FCoreDelegates::OnBeginFrame.AddLambda([this]() { OnBeginFrame_GameThread(); });
		EndFrameRTDelegate = FCoreDelegates::OnEndFrameRT.AddLambda([this]() { OnEndFrame_RenderThread(); });
	}
}

void UCinePrestreamingGraphNode::OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	if (!ActivePipeline.IsValid())
	{
		return;
	}

	// Force a final render-thread EndFrame to ensure the last frame's data is captured.
	// This must come before FlushRenderingCommands.
	OnEndFrame_GameThread();
	FlushRenderingCommands();

	// Unregister frame delegates — recording is complete.
	FCoreDelegates::OnBeginFrame.Remove(BeginFrameDelegate);
	FCoreDelegates::OnEndFrameRT.Remove(EndFrameRTDelegate);
	BeginFrameDelegate.Reset();
	EndFrameRTDelegate.Reset();

	// Restore any show flags we disabled.
	if (bShowFlagsDisabled)
	{
		for (const FString& Flag : CinePrestreamingGraphNode::GetShowFlagsToDisable())
		{
			UKismetSystemLibrary::ExecuteConsoleCommand(nullptr, Flag + TEXT(" 1"), nullptr);
		}
		bShowFlagsDisabled = false;
	}

	// Build and save prestreaming data assets from the collected frame data.
	CreateAssetsFromData(InPipeline, InPrimaryJobEvaluatedGraph);

	ResetStaticState();
}

void UCinePrestreamingGraphNode::OnAllFramesFinalizedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	// Safety cleanup in case the pipeline was shut down early before OnAllFramesSubmitted ran.
	if (ActivePipeline.IsValid())
	{
		FCoreDelegates::OnBeginFrame.Remove(BeginFrameDelegate);
		FCoreDelegates::OnEndFrameRT.Remove(EndFrameRTDelegate);
		BeginFrameDelegate.Reset();
		EndFrameRTDelegate.Reset();

		if (bShowFlagsDisabled)
		{
			for (const FString& Flag : CinePrestreamingGraphNode::GetShowFlagsToDisable())
			{
				UKismetSystemLibrary::ExecuteConsoleCommand(nullptr, Flag + TEXT(" 1"), nullptr);
			}
		}

		ResetStaticState();
	}
}

bool UCinePrestreamingGraphNode::IsFinishedWritingToDiskImpl() const
{
	// Assets are created synchronously in OnAllFramesSubmittedImpl.
	return true;
}

void UCinePrestreamingGraphNode::OnBeginFrame_GameThread()
{
	if (!ActivePipeline.IsValid())
	{
		return;
	}

	// Only record during active frame production.
	if (UMovieGraphBlueprintLibrary::GetPipelineState(ActivePipeline.Get()) != EMovieRenderPipelineState::ProducingFrames)
	{
		return;
	}

	const int32 ActiveShotIndex = ActivePipeline->GetCurrentShotIndex();
	if (ActiveShotIndex < 0 || !ActivePipeline->GetActiveShotList().IsValidIndex(ActiveShotIndex))
	{
		return;
	}

	if (PrevActiveShotIndex != ActiveShotIndex)
	{
		check(SegmentData.IsValidIndex(ActiveShotIndex));
		SegmentData[ActiveShotIndex].bValid = true;
		PrevActiveShotIndex = ActiveShotIndex;
	}

	// Shot-relative frame number in tick resolution.
	// CurrentTimeInRoot is updated each engine frame by the MRG time step.
	// InitialTimeInRoot is the immutable shot start time assigned by MRG.
	const FMoviePipelineCameraCutInfo& ShotInfo = ActivePipeline->GetActiveShotList()[ActiveShotIndex]->ShotInfo;
	const FFrameNumber FrameNumber = ShotInfo.CurrentTimeInRoot.FrameNumber - ShotInfo.InitialTimeInRoot.FrameNumber;

	// Activate recording buffers on the render thread for this engine tick.
	ENQUEUE_RENDER_COMMAND(CinePrestreaming_SetRecordBuffer)(
		[ShotIndex = (uint32)ActiveShotIndex, FrameNumber](FRHICommandListImmediate& RHICmdList)
		{
			const uint64 Handle = ((uint64)ShotIndex << 32) | (uint64)FrameNumber.Value;
			GetRendererModule().SetVirtualTextureRequestRecordBuffer(Handle);
			GetRendererModule().SetNaniteRequestRecordBuffer(Handle);
		});
}

void UCinePrestreamingGraphNode::OnEndFrame_GameThread()
{
	ENQUEUE_RENDER_COMMAND(CinePrestreaming_EndFrame)(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			OnEndFrame_RenderThread();
		});
}

void UCinePrestreamingGraphNode::OnEndFrame_RenderThread()
{
	// Read VT request data recorded this engine tick.
	{
		TMap<uint64, uint32> VirtualTextureRequestData;
		const uint64 Handle = GetRendererModule().GetVirtualTextureRequestRecordBuffer(VirtualTextureRequestData);
		if (Handle != ~0ull)
		{
			const uint32 ShotIndex = (uint32)(Handle >> 32);
			const FFrameNumber FrameNumber((int32)(Handle & 0xffffffff));
			if (SegmentData.IsValidIndex(ShotIndex))
			{
				// Overwrite with the latest data for this frame number (handles temporal sample repeats
				// and motion-blur frame-0→frame-1→frame-0 restart patterns; each overwrite is additive).
				SegmentData[ShotIndex].FrameData.FindOrAdd(FrameNumber).VirtualTextureRequestData = VirtualTextureRequestData;
			}
		}
	}

	// Read Nanite request data recorded this engine tick.
	{
		TMap<uint64, uint32> NaniteRequestData;
		const uint64 Handle = GetRendererModule().GetNaniteRequestRecordBuffer(NaniteRequestData);
		if (Handle != ~0ull)
		{
			const uint32 ShotIndex = (uint32)(Handle >> 32);
			const FFrameNumber FrameNumber((int32)(Handle & 0xffffffff));
			if (SegmentData.IsValidIndex(ShotIndex))
			{
				SegmentData[ShotIndex].FrameData.FindOrAdd(FrameNumber).NaniteRequestData = NaniteRequestData;
			}
		}
	}
}

void UCinePrestreamingGraphNode::CreateAssetsFromData(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph)
{
	// Fetch the global output settings node for resolution.
	constexpr bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		InPrimaryJobEvaluatedGraph->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	// Fetch the evaluated version of our node to read the actual (possibly overridden) property values.
	// Falls back to the CDO (this) if the node isn't present in the evaluated config.
	const UCinePrestreamingGraphNode* EvaluatedNode =
		InPrimaryJobEvaluatedGraph->GetSettingForBranch<UCinePrestreamingGraphNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	if (!EvaluatedNode)
	{
		EvaluatedNode = this;
	}

	FString FileNameFormatString = EvaluatedNode->PackageDirectory.Path;

	// Strip frame number tokens — we want one asset per shot, not one per frame.
	constexpr bool bIncludeRenderPass = false;
	constexpr bool bTestFrameNumber = false;
	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

	// Intermediate structure: segments that share the same resolved output path can be merged.
	struct FOutputSegment
	{
		FString FilePath;
		FCollectedData Data;
		int32 ShotIndex;
		UMovieScene* MovieScene;
		TRange<FFrameNumber> Range;
	};

	TArray<FOutputSegment> FinalSegments;

	for (int32 SegmentIndex = 0; SegmentIndex < SegmentData.Num(); SegmentIndex++)
	{
		const FCollectedData& Segment = SegmentData[SegmentIndex];
		if (!Segment.bValid || !InPipeline->GetActiveShotList().IsValidIndex(SegmentIndex))
		{
			continue;
		}

		const TObjectPtr<UMoviePipelineExecutorShot>& Shot = InPipeline->GetActiveShotList()[SegmentIndex];

		// Resolve the output path for this shot using MRG token substitution.
		FMovieGraphFilenameResolveParams ResolveParams;
		ResolveParams.InitializationTime = InPipeline->GetInitializationTime();
		ResolveParams.InitializationTimeOffset = InPipeline->GetInitializationTimeOffset();
		ResolveParams.Job = InPipeline->GetCurrentJob();
		ResolveParams.Shot = Shot;
		ResolveParams.EvaluatedConfig = InPrimaryJobEvaluatedGraph;
		ResolveParams.Version = Shot->ShotInfo.VersionNumber;
		ResolveParams.FileNameFormatOverrides =
		{
			{TEXT("render_pass"), TEXT("PrestreamingData")},
			{TEXT("ext"), TEXT("")},
		};

		FMovieGraphResolveArgs FinalFormatArgs;
		FString FinalFilePath = UMovieGraphBlueprintLibrary::ResolveFilenameFormatArguments(FileNameFormatString, ResolveParams, FinalFormatArgs);

		// Strip the trailing "." the resolver appends (intended for file extensions).
		if (FinalFilePath.EndsWith(TEXT(".")))
		{
			FinalFilePath.LeftChopInline(1);
		}
		FPaths::NormalizeDirectoryName(FinalFilePath);

		// Check if another segment resolved to the same path (merge into it).
		FOutputSegment* OutputSegment = nullptr;
		for (FOutputSegment& Existing : FinalSegments)
		{
			if (Existing.FilePath == FinalFilePath)
			{
				OutputSegment = &Existing;
				break;
			}
		}

		if (!OutputSegment)
		{
			FinalSegments.AddDefaulted();
			OutputSegment = &FinalSegments.Last();
			OutputSegment->FilePath = FinalFilePath;
			OutputSegment->ShotIndex = SegmentIndex;

			// Use the leaf-most movie scene (the shot's own sequence).
			const TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode>& Node = Shot->ShotInfo.SubSectionHierarchy;
			OutputSegment->MovieScene = Node.IsValid() ? Node->MovieScene.Get() : nullptr;
			OutputSegment->Range = UE::MovieScene::ConvertToDiscreteRange(
				Shot->ShotInfo.OuterToInnerTransform.ComputeTraversedHull(Shot->ShotInfo.TotalOutputRangeRoot));
		}

		OutputSegment->Data.FrameData.Append(Segment.FrameData);
	}

	TArray<FMoviePipelineCinePrestreamingGeneratedData> GeneratedData;

	for (FOutputSegment& Segment : FinalSegments)
	{
		TObjectPtr<UCinePrestreamingData> DataTarget = NewObject<UCinePrestreamingData>();

		if (OutputSettingNode)
		{
			DataTarget->RecordedResolution = OutputSettingNode->OutputResolution.Resolution;
		}
		DataTarget->RecordedTime = FDateTime::UtcNow();

		int32 FrameIndex = 0;
		int32 VirtualTextureMergeFrameIndex = 0;
		int32 NaniteMergeFrameIndex = 0;
		FFrameData AccumulatedFrameData;

		for (const TPair<FFrameNumber, FFrameData>& FrameData : Segment.Data.FrameData)
		{
			const bool bWriteFrameData = FrameIndex >= EvaluatedNode->StartFrame &&
				(FrameIndex <= EvaluatedNode->EndFrame || EvaluatedNode->EndFrame == 0);
			// Write a sentinel null entry at EndFrame+1 so playback knows not to repeat past the end.
			const bool bWriteFrameTime = bWriteFrameData || FrameIndex == EvaluatedNode->EndFrame + 1;

			++FrameIndex;

			if (bWriteFrameTime)
			{
				DataTarget->Times.Add(FrameData.Key);
				DataTarget->VirtualTextureDatas.AddDefaulted();
				DataTarget->NaniteDatas.AddDefaulted();
			}

			if (!bWriteFrameData)
			{
				continue;
			}

			if (EvaluatedNode->bVirtualTextures)
			{
				if (EvaluatedNode->bMergeFrames)
				{
					for (const TPair<uint64, uint32>& Pair : FrameData.Value.VirtualTextureRequestData)
					{
						uint32& Priority = AccumulatedFrameData.VirtualTextureRequestData.FindOrAdd(Pair.Key);
						Priority = FMath::Max(Priority, Pair.Value);
					}

					const int32 NumAccumulated = DataTarget->VirtualTextureDatas.Num() - VirtualTextureMergeFrameIndex;
					if (NumAccumulated >= EvaluatedNode->FrameCountMergeThreshold ||
						AccumulatedFrameData.VirtualTextureRequestData.Num() >= EvaluatedNode->VirtualTextureRequestMergeThreshold)
					{
						GetRendererModule().PackVirtualTextureRequestsToStream(
							AccumulatedFrameData.VirtualTextureRequestData,
							DataTarget->VirtualTextureDatas[VirtualTextureMergeFrameIndex].RequestData);
						AccumulatedFrameData.VirtualTextureRequestData.Reset();
						VirtualTextureMergeFrameIndex = DataTarget->VirtualTextureDatas.Num();
					}
				}
				else
				{
					GetRendererModule().PackVirtualTextureRequestsToStream(
						FrameData.Value.VirtualTextureRequestData,
						DataTarget->VirtualTextureDatas.Last().RequestData);
				}
			}

			if (EvaluatedNode->bNanite)
			{
				if (EvaluatedNode->bMergeFrames)
				{
					for (const TPair<uint64, uint32>& Pair : FrameData.Value.NaniteRequestData)
					{
						uint32& Priority = AccumulatedFrameData.NaniteRequestData.FindOrAdd(Pair.Key);
						Priority = FMath::Max(Priority, Pair.Value);
					}

					const int32 NumAccumulated = DataTarget->NaniteDatas.Num() - NaniteMergeFrameIndex;
					if (NumAccumulated >= EvaluatedNode->FrameCountMergeThreshold ||
						AccumulatedFrameData.NaniteRequestData.Num() >= EvaluatedNode->NaniteRequestMergeThreshold)
					{
						GetRendererModule().PackNaniteRequestsToStream(
							AccumulatedFrameData.NaniteRequestData,
							DataTarget->NaniteDatas[NaniteMergeFrameIndex].RequestData);
						AccumulatedFrameData.NaniteRequestData.Reset();
						NaniteMergeFrameIndex = DataTarget->NaniteDatas.Num();
					}
				}
				else
				{
					GetRendererModule().PackNaniteRequestsToStream(
						FrameData.Value.NaniteRequestData,
						DataTarget->NaniteDatas.Last().RequestData);
				}
			}
		}

		// Flush any remaining accumulated merge data.
		if (AccumulatedFrameData.VirtualTextureRequestData.Num())
		{
			GetRendererModule().PackVirtualTextureRequestsToStream(
				AccumulatedFrameData.VirtualTextureRequestData,
				DataTarget->VirtualTextureDatas[VirtualTextureMergeFrameIndex].RequestData);
		}
		if (AccumulatedFrameData.NaniteRequestData.Num())
		{
			GetRendererModule().PackNaniteRequestsToStream(
				AccumulatedFrameData.NaniteRequestData,
				DataTarget->NaniteDatas[NaniteMergeFrameIndex].RequestData);
		}

		FMoviePipelineCinePrestreamingGeneratedData NewData;
		NewData.StreamingData = DataTarget;

		ULevelSequence* LevelSequence = Segment.MovieScene
			? Segment.MovieScene->GetTypedOuter<ULevelSequence>()
			: nullptr;
		NewData.AssetName = LevelSequence
			? LevelSequence->GetName() + TEXT("_PreStream")
			: TEXT("PreStream");
		NewData.PackagePath = Segment.FilePath;
		NewData.MovieScene = Segment.MovieScene;
		NewData.Range = Segment.Range;

		GeneratedData.Add(NewData);
	}

	// Packages can only be saved in the editor. Save before modifying the target sequence
	// because saving duplicates assets into a package (which updates the GeneratedData pointers).
#if WITH_EDITOR
	if (UCinePrestreamingEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCinePrestreamingEditorSubsystem>())
	{
		Subsystem->CreatePackagesFromGeneratedData(/*InOut*/GeneratedData);
	}
#endif

	if (EvaluatedNode->bModifyTargetSequence)
	{
		ModifyTargetSequences(GeneratedData);
	}

	OnGenerateData.Broadcast(GeneratedData);
}

void UCinePrestreamingGraphNode::ModifyTargetSequences(const TArray<FMoviePipelineCinePrestreamingGeneratedData>& InData)
{
	for (const FMoviePipelineCinePrestreamingGeneratedData& Data : InData)
	{
		if (!Data.MovieScene)
		{
			continue;
		}

		const bool bReadOnly = Data.MovieScene->IsReadOnly();
		Data.MovieScene->SetReadOnly(false);

		UMovieSceneCinePrestreamingTrack* PrestreamingTrack = Data.MovieScene->FindTrack<UMovieSceneCinePrestreamingTrack>();
		if (!PrestreamingTrack)
		{
			PrestreamingTrack = Data.MovieScene->AddTrack<UMovieSceneCinePrestreamingTrack>();
		}

		// Remove existing sections to avoid stale references or incorrect durations.
		while (PrestreamingTrack->GetAllSections().Num() > 0)
		{
			PrestreamingTrack->RemoveSectionAt(0);
		}

		UMovieSceneCinePrestreamingSection* Section =
			Cast<UMovieSceneCinePrestreamingSection>(PrestreamingTrack->CreateNewSection());
		PrestreamingTrack->AddSection(*Section);

		Section->SetPrestreamingAsset(Data.StreamingData);
		Section->SetRange(Data.Range);

		Data.MovieScene->SetReadOnly(bReadOnly);
	}
}

void UCinePrestreamingGraphNode::ResetStaticState()
{
	ActivePipeline.Reset();
	SegmentData.Reset();
	PrevActiveShotIndex = -1;
	bShowFlagsDisabled = false;
}
