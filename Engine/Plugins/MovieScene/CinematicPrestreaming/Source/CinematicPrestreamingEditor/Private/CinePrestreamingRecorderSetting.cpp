// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinePrestreamingRecorderSetting.h"

#include "EngineModule.h"
#include "Kismet/KismetSystemLibrary.h"
#include "LevelSequence.h"
#include "MoviePipeline.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "Misc/Paths.h"
#include "MoviePipelineQueue.h"
#include "MoviePipelineUtils.h"
#include "MovieScene.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "Tracks/MovieSceneCinePrestreamingTrack.h"
#include "Sections/MovieSceneCinePrestreamingSection.h"

#if WITH_EDITOR
#include "Editor.h"
#include "CinePrestreamingEditorSubsystem.h"
#endif

UCinePrestreamingRecorderSetting::UCinePrestreamingRecorderSetting()
	: PrevActiveShotIndex(-1)
{
	PackageDirectory.Path = TEXT("/Game/Cinematics/Prestreaming/{sequence_name}/{shot_name}/");

	ShowFlagsToDisable.Add("ShowFlag.PostProcessing");
	ShowFlagsToDisable.Add("ShowFlag.Lighting");
	ShowFlagsToDisable.Add("ShowFlag.Atmosphere");
	ShowFlagsToDisable.Add("ShowFlag.VolumetricFog");
	ShowFlagsToDisable.Add("ShowFlag.VolumetricLightmap");
	ShowFlagsToDisable.Add("ShowFlag.LumenGlobalIllumination");
	ShowFlagsToDisable.Add("ShowFlag.LumenReflections");
	ShowFlagsToDisable.Add("ShowFlag.AmbientOcclusion");
	ShowFlagsToDisable.Add("ShowFlag.DistanceFieldAO");
}

void UCinePrestreamingRecorderSetting::SetupForPipelineImpl(UMoviePipeline* InPipeline)
{
	// Subscribe to BeginFrame to capture the start of rendering on game thread. We may actually render multiple times to produce a single output Frame.
	BeginFrameDelegate = FCoreDelegates::OnBeginFrame.AddLambda([this]() { OnBeginFrame_GameThread(); });
	// Subscribe to EndFrameRT to capture the end of rendering on the render thread. This should be called after all rendering for a single output Frame is complete.
	EndFrameDelegate = FCoreDelegates::OnEndFrameRT.AddLambda([this]() { OnEndFrame_RenderThread(); });

	SegmentData.Reset();
	PrevActiveShotIndex = ~0u;

	if (bDisableAdvanceRenderFeatures)
	{
		for (const FString& Flag : ShowFlagsToDisable)
		{
			FString CombinedStr = Flag + TEXT(" 0");
			UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), CombinedStr, nullptr);
		}
	}
}

void UCinePrestreamingRecorderSetting::TeardownForPipelineImpl(UMoviePipeline* InPipeline)
{
	// Force an EndFrame to ensure that all recording is captured. This enqueues a command onto the render
	// thread, so it needs to come before the flush.
	OnEndFrame_GameThread();

	// Ensure all render thread work is complete.
	FlushRenderingCommands();

	// End listening.
	FCoreDelegates::OnBeginFrame.Remove(BeginFrameDelegate);
	FCoreDelegates::OnEndFrameRT.Remove(EndFrameDelegate);

	// Save our data asset to disk.
	CreateAssetsFromData();

	SegmentData.Reset();

	if (bDisableAdvanceRenderFeatures)
	{
		for (const FString& Flag : ShowFlagsToDisable)
		{
			FString CombinedStr = Flag + TEXT(" 1");
			UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), CombinedStr, nullptr);
		}
	}
}
	
void UCinePrestreamingRecorderSetting::OnBeginFrame_GameThread()
{
	// Don't capture any data after we finish rendering.
	if (GetPipeline()->GetPipelineState() != EMovieRenderPipelineState::ProducingFrames)
	{
		return;
	}

	const int32 ActiveShotIndex = GetPipeline()->GetCurrentShotIndex();
	if (ActiveShotIndex < 0)
	{
		return;
	}

	if (PrevActiveShotIndex != ActiveShotIndex)
	{
		// Only allocate the SegmentData array once.
		// Need to avoid any reallocation after the render thread could be writing it.
		const int32 ActiveShotNum = GetPipeline()->GetActiveShotList().Num();
		check(SegmentData.Num() == 0 || SegmentData.Num() == ActiveShotNum);
		if (SegmentData.Num() == 0)
		{
			SegmentData.AddDefaulted(ActiveShotNum);
		}

		SegmentData[ActiveShotIndex].bValid = true;
		SegmentData[ActiveShotIndex].OutputState = GetPipeline()->GetOutputState();
		SegmentData[ActiveShotIndex].InitialShotTick = GetPipeline()->GetActiveShotList()[ActiveShotIndex]->ShotInfo.CurrentTickInRoot;

		PrevActiveShotIndex = ActiveShotIndex;
	}

	// Be warned that frame time can have repeats and isn't always incrementing, ie: slow-mo will have this
	// number be the same for many frames, and the start of each shot will jump forward and back to
	// emulate motion blur, ie: frame 0 -> frame 1 -> frame 0 again. 
	// So when building data it needs to be additive.
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[ActiveShotIndex];
	const FFrameNumber FrameNumber = CurrentShot->ShotInfo.CurrentTickInRoot - SegmentData[ActiveShotIndex].InitialShotTick;
	
	// Allocate virtual texture recording page buffer for this frame on the render thread.
	ENQUEUE_RENDER_COMMAND(SetRecordBuffer)(
		[ShotIndex = (uint32)PrevActiveShotIndex, FrameNumber](FRHICommandListImmediate& RHICmdList)
		{
			const uint64 Handle = ((uint64)ShotIndex << 32) | (uint64)FrameNumber.Value;
			GetRendererModule().SetVirtualTextureRequestRecordBuffer(Handle);
			GetRendererModule().SetNaniteRequestRecordBuffer(Handle);
		});
}

void UCinePrestreamingRecorderSetting::OnEndFrame_GameThread()
{
	ENQUEUE_RENDER_COMMAND(EndFrame)(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			OnEndFrame_RenderThread();
		});
}

void UCinePrestreamingRecorderSetting::OnEndFrame_RenderThread()
{
	// Finalize request collection for this frame.
	{
		// We avoid the need for a critical section by assuming that this is the only code accessing SegmentData during the recording phase.
		TMap<uint64, uint32> VirtualTextureRequestData;
		const uint64 Handle = GetRendererModule().GetVirtualTextureRequestRecordBuffer(VirtualTextureRequestData);
		if (Handle != ~0ull)
		{
			const uint32 ShotIndex = (uint32)(Handle >> 32);
			const FFrameNumber FrameNumber((int32)(Handle & 0xffffffff));
			// Only store the latest matching container.
			SegmentData[ShotIndex].FrameData.FindOrAdd(FrameNumber).VirtualTextureRequestData = VirtualTextureRequestData;
		}
	}

	{
		TMap<uint64, uint32> NaniteRequestData;
		const uint64 Handle = GetRendererModule().GetNaniteRequestRecordBuffer(NaniteRequestData);
		if (Handle != ~0ull)
		{
			const uint32 ShotIndex = (uint32)(Handle >> 32);
			const FFrameNumber FrameNumber((int32)(Handle & 0xffffffff));
			// Only store the latest matching container.
			SegmentData[ShotIndex].FrameData.FindOrAdd(FrameNumber).NaniteRequestData = NaniteRequestData;
		}
	}
}

void UCinePrestreamingRecorderSetting::CreateAssetsFromData()
{
	UMoviePipelineOutputSetting* OutputSetting = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	FString FileNameFormatString = PackageDirectory.Path;

	const bool bIncludeRenderPass = false;
	const bool bTestFrameNumber = false;
 	UE::MoviePipeline::ValidateOutputFormatString(FileNameFormatString, bIncludeRenderPass, bTestFrameNumber);
	// Strip any frame number tags so we don't get one asset per frame.
	UE::MoviePipeline::RemoveFrameNumberFormatStrings(FileNameFormatString, true);

	struct FOutputSegment
	{
		FString FilePath;
		FCollectedData Data;
		int32 ShotIndex;
		UMovieScene* MovieScene;
		TRange<FFrameNumber> Range;
	};

	TArray<FOutputSegment> FinalSegments;

	for (int32 SegmentIndex = 0; SegmentIndex < SegmentData.Num(); SegmentIndex++)
	{
		const FCollectedData& Segment = SegmentData[SegmentIndex];
		if (!Segment.bValid)
		{
			continue;
		}

		// Generate a filename for this output file.
		TMap<FString, FString> FormatOverrides;
		FormatOverrides.Add(TEXT("render_pass"), TEXT("PrestreamingData"));
		FormatOverrides.Add(TEXT("ext"), TEXT(""));

		const FMoviePipelineCameraCutInfo& ShotInfo = GetPipeline()->GetActiveShotList()[Segment.OutputState.ShotIndex]->ShotInfo;

		// If they don't specify a shot, take it out of the format string to avoid having a folder named "no shot"
		TSharedPtr<MoviePipeline::FCameraCutSubSectionHierarchyNode> Node = ShotInfo.SubSectionHierarchy;
		if (!Node.IsValid() || !Node->GetParent().IsValid())
		{
			FormatOverrides.Add(TEXT("shot_name"), TEXT(""));
		}
		

		FMoviePipelineFormatArgs FinalFormatArgs;
		FString FinalFilePath;
		GetPipeline()->ResolveFilenameFormatArguments(FileNameFormatString, FormatOverrides, FinalFilePath, FinalFormatArgs, &Segment.OutputState);

		// Remove the final "." ResolveFilenameFormatArgs adds
		if (FinalFilePath.EndsWith(TEXT(".")))
		{
			FinalFilePath.LeftChopInline(1);
		}
		FPaths::NormalizeDirectoryName(FinalFilePath);

		// Look to see if we already have a output file to append this to.
		FOutputSegment* OutputSegment = nullptr;
		for (int32 Index = 0; Index < FinalSegments.Num(); Index++)
		{
			if (FinalSegments[Index].FilePath == FinalFilePath)
			{
				OutputSegment = &FinalSegments[Index];
				break;
			}
		}

		if (!OutputSegment)
		{
			FinalSegments.AddDefaulted();
			OutputSegment = &FinalSegments[FinalSegments.Num() - 1];
			OutputSegment->FilePath = FinalFilePath;
			OutputSegment->ShotIndex = Segment.OutputState.ShotIndex;

			// We only use the leafmost moviescene as that's the one that actually gets modified.
			OutputSegment->MovieScene = Node.IsValid() ? Node->MovieScene.Get() : nullptr;
			OutputSegment->Range = UE::MovieScene::ConvertToDiscreteRange(ShotInfo.OuterToInnerTransform.ComputeTraversedHull(ShotInfo.TotalOutputRangeRoot));
		}

		// Convert our samples and append them to the existing array
		OutputSegment->Data.FrameData.Append(Segment.FrameData);
	}

	TArray<FMoviePipelineCinePrestreamingGeneratedData> GeneratedData;
	// Now that all of the segments have been merged, write out only the final files.
	for (FOutputSegment& Segment : FinalSegments)
	{
		TObjectPtr<UCinePrestreamingData> DataTarget = NewObject<UCinePrestreamingData>();
		DataTarget->RecordedResolution = OutputSetting->OutputResolution;
		DataTarget->RecordedTime = FDateTime::UtcNow();

		int32 FrameIndex = 0;
		int32 VirtualTextureMergeFrameIndex = 0;
		int32 NaniteMergeFrameIndex = 0;
		FFrameData AccumulatedFrameData;

		for (TPair< FFrameNumber, FFrameData > const& FrameData : Segment.Data.FrameData)
		{
			// Only write out data within the specified range.
			const bool bWriteFrameData = FrameIndex >= StartFrame && (FrameIndex <= EndFrame || EndFrame == 0);
			// Write a null entry at the end of the data so that during playback we don't keep repeating the last frame.
			const bool bWriteFrameTime = bWriteFrameData || FrameIndex == EndFrame + 1;

			++FrameIndex;

			if (bWriteFrameTime)
			{
				DataTarget->Times.Add(FrameData.Key);
				DataTarget->VirtualTextureDatas.AddDefaulted();
				DataTarget->NaniteDatas.AddDefaulted();
			}

			if (!bWriteFrameData)
			{
				continue;
			}

			if (bVirtualTextures)
			{
				if (bMergeFrames)
				{
					// Merge this frame's data with any accumulated previous frames.
					for (TPair<uint64, uint32> const& Pair : FrameData.Value.VirtualTextureRequestData)
					{
						uint32& Priority = AccumulatedFrameData.VirtualTextureRequestData.FindOrAdd(Pair.Key);
						Priority = FMath::Max(Priority, Pair.Value);
					}
					
					// Flush and pack requests once we reach a merge threshold.
					const int32 NumAccumulatedFrames = DataTarget->VirtualTextureDatas.Num() - VirtualTextureMergeFrameIndex;

					if (NumAccumulatedFrames >= FrameCountMergeThreshold || AccumulatedFrameData.VirtualTextureRequestData.Num() >= VirtualTextureRequestMergeThreshold)
					{
						GetRendererModule().PackVirtualTextureRequestsToStream(AccumulatedFrameData.VirtualTextureRequestData, DataTarget->VirtualTextureDatas[VirtualTextureMergeFrameIndex].RequestData);
						AccumulatedFrameData.VirtualTextureRequestData.Reset();
						VirtualTextureMergeFrameIndex = DataTarget->VirtualTextureDatas.Num();
					}
				}
				else
				{
					// Pack requests to final runtime format.
					GetRendererModule().PackVirtualTextureRequestsToStream(FrameData.Value.VirtualTextureRequestData, DataTarget->VirtualTextureDatas.Last().RequestData);
				}
			}
					
			if (bNanite)
			{
				if (bMergeFrames)
				{
					// Merge this frame's data with any accumulated previous frames.
					for (TPair<uint64, uint32> const& Pair : FrameData.Value.NaniteRequestData)
					{
						uint32& Priority = AccumulatedFrameData.NaniteRequestData.FindOrAdd(Pair.Key);
						Priority = FMath::Max(Priority, Pair.Value);
					}

					// Flush and pack requests once we reach a merge threshold.
					const int32 NumAccumulatedFrames = DataTarget->NaniteDatas.Num() - NaniteMergeFrameIndex;

					if (NumAccumulatedFrames >= FrameCountMergeThreshold || AccumulatedFrameData.NaniteRequestData.Num() >= NaniteRequestMergeThreshold)
					{
						GetRendererModule().PackNaniteRequestsToStream(AccumulatedFrameData.NaniteRequestData, DataTarget->NaniteDatas[NaniteMergeFrameIndex].RequestData);
						AccumulatedFrameData.NaniteRequestData.Reset();
						NaniteMergeFrameIndex = DataTarget->NaniteDatas.Num();
					}
				}
				else
				{
					// Pack requests to final runtime format.
					GetRendererModule().PackNaniteRequestsToStream(FrameData.Value.NaniteRequestData, DataTarget->NaniteDatas.Last().RequestData);
				}
			}
		}

		// Flush any remaining accumulated frame data.
		if (AccumulatedFrameData.VirtualTextureRequestData.Num())
		{
			GetRendererModule().PackVirtualTextureRequestsToStream(AccumulatedFrameData.VirtualTextureRequestData, DataTarget->VirtualTextureDatas[VirtualTextureMergeFrameIndex].RequestData);
		}
		if (AccumulatedFrameData.NaniteRequestData.Num())
		{
			GetRendererModule().PackNaniteRequestsToStream(AccumulatedFrameData.NaniteRequestData, DataTarget->NaniteDatas[NaniteMergeFrameIndex].RequestData);
		}

		FMoviePipelineCinePrestreamingGeneratedData NewData;
		NewData.StreamingData = DataTarget;

		ULevelSequence* LevelSequence = Segment.MovieScene ? Segment.MovieScene->GetTypedOuter<ULevelSequence>() : nullptr;
		NewData.AssetName = LevelSequence ? LevelSequence->GetName() + TEXT("_PreStream") : TEXT("PreStream");
		NewData.PackagePath = Segment.FilePath;
		NewData.MovieScene = Segment.MovieScene;
		NewData.Range = Segment.Range;
		GeneratedData.Add(NewData);
	}

	// Packages can only be saved in the editor and need to be saved
	// before modifying the target sequence, as saving duplicates them
	// into a package (which will modify GeneratedData to point to the new one)
#if WITH_EDITOR
	if (UCinePrestreamingEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCinePrestreamingEditorSubsystem>())
	{
		Subsystem->CreatePackagesFromGeneratedData(/*InOut*/GeneratedData);
	}
#endif
	if (bModifyTargetSequence)
	{
		ModifyTargetSequences(GeneratedData);
	}


	OnGenerateData.Broadcast(GeneratedData);
}

void UCinePrestreamingRecorderSetting::ModifyTargetSequences(const TArray<FMoviePipelineCinePrestreamingGeneratedData>& InData)
{
	for (const FMoviePipelineCinePrestreamingGeneratedData& Data : InData)
	{
		if (!Data.MovieScene)
		{
			continue;
		}

		const bool bReadOnly = Data.MovieScene->IsReadOnly();
		Data.MovieScene->SetReadOnly(false);

		// If there's an existing track we'll edit that instead.
		UMovieSceneCinePrestreamingTrack* PrestreamingTrack = Data.MovieScene->FindTrack<UMovieSceneCinePrestreamingTrack>();
		if (!PrestreamingTrack)
		{
			PrestreamingTrack = Data.MovieScene->AddTrack<UMovieSceneCinePrestreamingTrack>();
		}

		// Remove any existing sections, as they may point to an old object, or have the wrong length, etc.
		while (PrestreamingTrack->GetAllSections().Num() > 0)
		{
			PrestreamingTrack->RemoveSectionAt(0);
		}

		// Now create a new section to configure
		UMovieSceneCinePrestreamingSection* Section = Cast<UMovieSceneCinePrestreamingSection>(PrestreamingTrack->CreateNewSection());
		PrestreamingTrack->AddSection(*Section);

		Section->SetPrestreamingAsset(Data.StreamingData);
		Section->SetRange(Data.Range);
		Data.MovieScene->SetReadOnly(bReadOnly);
	}
}


#if WITH_EDITOR

FText UCinePrestreamingRecorderSetting::GetFooterText(UMoviePipelineExecutorJob* InJob) const
{
	if (bDisableAdvanceRenderFeatures)
	{
		return NSLOCTEXT("MovieRenderPipeline", "CinePrestreamingSetting_FooterText", "Enabling the bDisableAdvanceRenderFeatures setting will disable rendering features not required to generate the Cinematic Prestreaming data. This makes renders significantly faster but results in a final image that is not useful.");
	}

	return FText();
}

#endif
