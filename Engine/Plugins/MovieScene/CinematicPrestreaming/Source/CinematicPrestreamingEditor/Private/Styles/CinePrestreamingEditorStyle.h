// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Paths.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"

#define IMAGE_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

/**
 * Implements the visual style of the cinematic prestreaming sequence editor.
 */
class FCinePrestreamingEditorStyle final : public FSlateStyleSet
{
public:
	
	FCinePrestreamingEditorStyle()
		: FSlateStyleSet("CinePrestreamingEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon48x48(48.0f, 48.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("MovieScene/CinematicPrestreaming/Resources"));

		Set("Sequencer.Tracks.CinePrestreaming_16", new IMAGE_BRUSH_SVG("CinePrestreaming_16", Icon16x16));
		Set("Sequencer.Tracks.CinePrestreaming_64", new IMAGE_BRUSH_SVG("CinePrestreaming_64", Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	virtual ~FCinePrestreamingEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static TSharedRef<FCinePrestreamingEditorStyle> Get()
	{
		if (!Singleton.IsValid())
		{
			Singleton = MakeShareable(new FCinePrestreamingEditorStyle);
		}
		return Singleton.ToSharedRef();
	}

private:

	static TSharedPtr<FCinePrestreamingEditorStyle> Singleton;
};

#undef IMAGE_BRUSH_SVG
