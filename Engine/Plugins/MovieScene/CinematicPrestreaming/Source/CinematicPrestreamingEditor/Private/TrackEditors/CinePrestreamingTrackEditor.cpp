// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CinePrestreamingTrackEditor.h"

#include "ScopedTransaction.h"
#include "Styles/CinePrestreamingEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencerSection.h"
#include "LevelSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "Sections/MovieSceneCinePrestreamingSection.h"
#include "SequencerSectionPainter.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Styling/StyleColors.h"
#include "Tracks/MovieSceneCinePrestreamingTrack.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CinePrestreamingTrackEditor"

struct FCinePrestreamingSection
	: public ISequencerSection
	, public TSharedFromThis<FCinePrestreamingSection>
{
	FCinePrestreamingSection(UMovieSceneCinePrestreamingSection* InSection)
		: WeakSection(InSection)
	{}

	UMovieSceneSection* GetSectionObject() override
	{
		return WeakSection.Get();
	}

	int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override
	{
		return InPainter.PaintSectionBackground();
	}

	float GetSectionHeight() const override
	{
		return 30.f;
	}

	TSharedRef<SWidget> GenerateSectionWidget() override
	{
		return SNew(SBox)
		.Padding(FMargin(4.f))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &FCinePrestreamingSection::GetVisibilityText)
					.ColorAndOpacity(this, &FCinePrestreamingSection::GetTextColor)
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &FCinePrestreamingSection::GetEvalOffsetText)
				]
			]

			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &FCinePrestreamingSection::GetLayerBarText)
				.AutoWrapText(true)
			]
		];
	}

	FText GetVisibilityText() const
	{
		UMovieSceneCinePrestreamingSection* Section = WeakSection.Get();
		if (Section && Section->GetPrestreamingAsset())
		{
			return FText::FromString(FString::Printf(TEXT("Asset: %s"), *Section->GetPrestreamingAsset()->GetName()));
		}

		return LOCTEXT("VisibilityText_NoAsset", "No Asset Specified");
	}

	FText GetEvalOffsetText() const
	{
		return FText();
	}

	FText GetLayerBarText() const
	{
		UMovieSceneCinePrestreamingSection* Section = WeakSection.Get();
		if (Section)
		{
			if (const UCinePrestreamingData* PrestreamingData = Section->GetPrestreamingAsset())
			{
				FDateTime CurTime = FDateTime::UtcNow();
				FTimespan DeltaSinceRecording = (CurTime - PrestreamingData->RecordedTime);
				FIntPoint Resolution = PrestreamingData->RecordedResolution;
				FFormatNamedArguments Args;
				Args.Add("FrameCount", PrestreamingData->Times.Num());
				Args.Add("ResX", Resolution.X);
				Args.Add("ResY", Resolution.Y);
				FString DeltaStrDays = DeltaSinceRecording.ToString(TEXT("%d"));
				FString DeltaStrHrs = DeltaSinceRecording.ToString(TEXT("%h"));
				Args.Add("DateDeltaD", FText::FromString(DeltaStrDays));
				Args.Add("DateDeltaH", FText::FromString(DeltaStrHrs));
				const FFrameTime OffsetTime = Section->GetOffsetTime().Get(FFrameTime());
				Args.Add("OffsetFrames", OffsetTime.FrameNumber.Value);

				return FText::Format(LOCTEXT("LayerBarFormatString", "Recorded Duration: {FrameCount} Frames, Resolution: ({ResX}x{ResY}), Recorded: {DateDeltaD} days, {DateDeltaH} hrs ago. Eval Offset: {OffsetFrames} Frames"), Args);
			}
		}
		return FText();
	}

	FSlateColor GetTextColor() const
	{
		return FStyleColors::Foreground;
	}

private:
	TWeakObjectPtr<UMovieSceneCinePrestreamingSection> WeakSection;
};


FCinePrestreamingTrackEditor::FCinePrestreamingTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{}

TSharedRef<ISequencerTrackEditor> FCinePrestreamingTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShared<FCinePrestreamingTrackEditor>(InSequencer);
}

FText FCinePrestreamingTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CinePrestreamingTrackEditor_DisplayName", "Cinematic Prestreaming");
}

bool FCinePrestreamingTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return InSequence ? InSequence->GetClass() == ULevelSequence::StaticClass() : false;
}

bool FCinePrestreamingTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneCinePrestreamingTrack::StaticClass();
}

const FSlateBrush* FCinePrestreamingTrackEditor::GetIconBrush() const
{
	return FCinePrestreamingEditorStyle::Get()->GetBrush("Sequencer.Tracks.CinePrestreaming_16");
}

TSharedRef<ISequencerSection> FCinePrestreamingTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	UMovieSceneCinePrestreamingSection* PrestreamingSection = Cast<UMovieSceneCinePrestreamingSection>(&SectionObject);
	check(SupportsType(SectionObject.GetOuter()->GetClass()) && PrestreamingSection != nullptr);

	return MakeShared<FCinePrestreamingSection>(PrestreamingSection);
}

void FCinePrestreamingTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTrack", "Cinematic Prestreaming"),
		LOCTEXT("AddTrackToolTip", "Adds a new track that can load streaming data used by the renderer to request things in advanced of needing them."),
		FSlateIcon(FCinePrestreamingEditorStyle::Get()->GetStyleSetName(), "Sequencer.Tracks.CinePrestreaming_16"),
		FUIAction(FExecuteAction::CreateRaw(this, &FCinePrestreamingTrackEditor::HandleAddTrack)));
}

TSharedPtr<SWidget> FCinePrestreamingTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return UE::Sequencer::MakeAddButton(
		LOCTEXT("AddCinePrestreaming_ButtonLabel", "Cinematic Prestreaming"),
		FOnGetContent::CreateSP(this, &FCinePrestreamingTrackEditor::BuildAddPrestreamingMenu, Track),
		Params.ViewModel);
}

UMovieSceneCinePrestreamingSection* FCinePrestreamingTrackEditor::AddNewSection(UMovieScene* MovieScene, UMovieSceneTrack* PrestreamingTrack)
{
	using namespace UE::MovieScene;

	const FScopedTransaction Transaction(LOCTEXT("AddCinePrestreamingSection_Transaction", "Add Cinematic Prestreaming"));
	PrestreamingTrack->Modify();

	UMovieSceneCinePrestreamingSection* PrestreamingSection = CastChecked<UMovieSceneCinePrestreamingSection>(PrestreamingTrack->CreateNewSection());

	PrestreamingSection->InitialPlacement(PrestreamingTrack->GetAllSections(), MovieScene->GetPlaybackRange().GetLowerBoundValue(), DiscreteSize(MovieScene->GetPlaybackRange()), true);
	PrestreamingTrack->AddSection(*PrestreamingSection);

	return PrestreamingSection;
}

void FCinePrestreamingTrackEditor::HandleAddTrack()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene == nullptr || FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddCinePrestreamingTrack_Transaction", "Add Cinematic Prestreaming Track"));
	FocusedMovieScene->Modify();

	UMovieSceneCinePrestreamingTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneCinePrestreamingTrack>();
	checkf(NewTrack, TEXT("Failed to create new cinematic prestreaming track."));

	UMovieSceneCinePrestreamingSection* NewSection = AddNewSection(FocusedMovieScene, NewTrack);
	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

TSharedRef<SWidget> FCinePrestreamingTrackEditor::BuildAddPrestreamingMenu(UMovieSceneTrack* PrestreamingTrack)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddNewSection", "New Section"),
		LOCTEXT("AddNewSection_Tip", "Add a new section"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(
			this, &FCinePrestreamingTrackEditor::HandleAddNewSection, PrestreamingTrack)));

	return MenuBuilder.MakeWidget();
}

void FCinePrestreamingTrackEditor::HandleAddNewSection(UMovieSceneTrack* PrestreamingTrack)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene)
	{
		UMovieSceneCinePrestreamingSection* NewSection = AddNewSection(FocusedMovieScene, PrestreamingTrack);

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		GetSequencer()->EmptySelection();
		GetSequencer()->SelectSection(NewSection);
		GetSequencer()->ThrobSectionSelection();
	}
}

#undef LOCTEXT_NAMESPACE
