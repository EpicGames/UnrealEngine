// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"

struct FBuildEditWidgetParams;

class FMenuBuilder;
class UMovieSceneCinePrestreamingSection;

/**
 * A sequencer track editor for cinematic prestreaming tracks.
 */
class FCinePrestreamingTrackEditor
	: public FMovieSceneTrackEditor
{
public:
	FCinePrestreamingTrackEditor(TSharedRef<ISequencer> InSequencer);

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

protected:
	/** ISequencerTrackEditor interface */
	virtual FText GetDisplayName() const override;
	TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	const FSlateBrush* GetIconBrush() const override;
	void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;

private:
	/** Adds a new section which spans the length of the owning movie scene. */
	UMovieSceneCinePrestreamingSection* AddNewSection(UMovieScene* MovieScene, UMovieSceneTrack* PrestreamingTrack);

	/** Handles when the add track menu item is activated. */
	void HandleAddTrack();

	/** Builds the add prestreaming menu which is displayed on the track. */
	TSharedRef<SWidget> BuildAddPrestreamingMenu(UMovieSceneTrack* PrestreamingTrack);

	/** Called to add a new section. */
	void HandleAddNewSection(UMovieSceneTrack* PrestreamingTrack);
};
