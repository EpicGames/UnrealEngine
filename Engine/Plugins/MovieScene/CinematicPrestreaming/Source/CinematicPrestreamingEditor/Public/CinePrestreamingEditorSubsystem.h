// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "CinePrestreamingEditorSubsystem.generated.h"

UCLASS(BlueprintType)
class CINEMATICPRESTREAMINGEDITOR_API UCinePrestreamingEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UCinePrestreamingEditorSubsystem() = default;

	UFUNCTION(BlueprintCallable, Category = "Cinematic Prestreaming|Editor")
	void CreatePackagesFromGeneratedData(TArray<struct FMoviePipelineCinePrestreamingGeneratedData>& InOutData);
};
