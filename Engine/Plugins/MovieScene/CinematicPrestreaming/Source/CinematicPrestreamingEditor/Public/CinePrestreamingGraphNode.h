// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Nodes/MovieGraphFileOutputNode.h"
#include "CinePrestreamingRecorderSetting.h"
#include "CinePrestreamingGraphNode.generated.h"

class UMovieGraphPipeline;
class UMovieGraphEvaluatedConfig;

/**
 * A Movie Render Graph output node that records virtual texture page requests and Nanite geometry
 * requests during rendering, then saves them as UCinePrestreamingData assets. These assets can
 * be used at runtime to pre-stream content before it is needed during cinematic playback.
 *
 * This node provides the same recording functionality as UCinePrestreamingRecorderSetting (the
 * Movie Render Queue version), but integrates with the Movie Render Graph pipeline instead.
 *
 * Note: Because delegates are registered lazily on the first completed output frame, the very
 * first output frame's streaming requests are not captured. This is acceptable because warmup
 * frames precede actual output frames and pre-establish the streaming state.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCinePrestreamingGraphNode : public UMovieGraphFileOutputNode
{
	GENERATED_BODY()

public:
	UCinePrestreamingGraphNode();

	// UMovieGraphNode Interface
	CINEMATICPRESTREAMINGEDITOR_API virtual EMovieGraphBranchRestriction GetBranchRestriction() const override;
#if WITH_EDITOR
	CINEMATICPRESTREAMINGEDITOR_API virtual FText GetNodeTitle(const bool bGetDescriptive) const override;
	CINEMATICPRESTREAMINGEDITOR_API virtual FText GetMenuCategory() const override;
	CINEMATICPRESTREAMINGEDITOR_API virtual FText GetKeywords() const override;
	CINEMATICPRESTREAMINGEDITOR_API virtual FLinearColor GetNodeTitleColor() const override;
#endif
	// ~UMovieGraphNode Interface

	// UMovieGraphFileOutputNode Interface
protected:
	CINEMATICPRESTREAMINGEDITOR_API virtual void OnReceiveImageDataImpl(UMovieGraphPipeline* InPipeline, UE::MovieGraph::FMovieGraphOutputMergerFrame* InRawFrameData, const TSet<FMovieGraphRenderDataIdentifier>& InMask) override;
	CINEMATICPRESTREAMINGEDITOR_API virtual void OnAllFramesSubmittedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	CINEMATICPRESTREAMINGEDITOR_API virtual void OnAllFramesFinalizedImpl(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph) override;
	CINEMATICPRESTREAMINGEDITOR_API virtual bool IsFinishedWritingToDiskImpl() const override;
	// ~UMovieGraphFileOutputNode Interface

public:
	/**
	 * Specifies the content browser directory where generated assets will be placed.
	 * Can contain format tokens such as {sequence_name} and {shot_name}.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ContentDir))
	FDirectoryPath PackageDirectory;

	/** Enable capture of virtual texture page requests. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Record")
	bool bVirtualTextures = true;

	/** Enable capture of Nanite geometry requests. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Record")
	bool bNanite = true;

	/** Automatically add the generated prestreaming track to the target sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process")
	bool bModifyTargetSequence = true;

	/** Enable merging of frame data. Reduces asset size when consecutive frames have high request coherence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations")
	bool bMergeFrames = false;

	/** Maximum number of frames to merge data across. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations", meta = (EditCondition = "bMergeFrames", UIMin = "1", ClampMin = "1"))
	int32 FrameCountMergeThreshold = 6;

	/** Threshold of virtual texture page requests. Frames are merged until this threshold is reached. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations", meta = (EditCondition = "bVirtualTextures && bMergeFrames", UIMin = "1", ClampMin = "1"))
	int32 VirtualTextureRequestMergeThreshold = 2000;

	/** Threshold of Nanite page requests. Frames are merged until this threshold is reached. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations", meta = (EditCondition = "bNanite && bMergeFrames", UIMin = "1", ClampMin = "1"))
	int32 NaniteRequestMergeThreshold = 500;

	/**
	 * Disable rendering features not required to generate prestreaming data (lighting, post processing, etc.).
	 * Makes recording significantly faster but produces unusable rendered images.
	 * Only enable this when using a dedicated recording graph with no image output nodes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations")
	bool bDisableAdvanceRenderFeatures = false;

	/** First frame to include in the recorded asset. Frames before this index are dropped. */
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	int32 StartFrame = 0;

	/** Last frame to include in the recorded asset. Frames after this index are dropped. Set to 0 to include all frames. */
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	int32 EndFrame = 0;

	/** Broadcast after asset generation completes, with the generated data. */
	UPROPERTY(BlueprintAssignable)
	FOnCinePrestreamingGenerateData OnGenerateData;

private:
	void OnBeginFrame_GameThread();
	void OnEndFrame_GameThread();
	void OnEndFrame_RenderThread();
	void CreateAssetsFromData(UMovieGraphPipeline* InPipeline, TObjectPtr<UMovieGraphEvaluatedConfig>& InPrimaryJobEvaluatedGraph);
	void ModifyTargetSequences(const TArray<FMoviePipelineCinePrestreamingGeneratedData>& InData);
	void ResetStaticState();

	// Per-frame collected data structures (same layout as MRQ recorder)
	struct FFrameData
	{
		TMap<uint64, uint32> VirtualTextureRequestData;
		TMap<uint64, uint32> NaniteRequestData;
	};

	struct FCollectedData
	{
		bool bValid = false;
		/** Map of shot-relative FFrameNumber to collected streaming requests. */
		TMap<FFrameNumber, FFrameData> FrameData;
	};

	// The pipeline generates many CDO instances throughout execution. Static data provides
	// shared state across all CDO calls for the lifetime of a single pipeline run.
	inline static TWeakObjectPtr<UMovieGraphPipeline> ActivePipeline;
	inline static FDelegateHandle BeginFrameDelegate;
	inline static FDelegateHandle EndFrameRTDelegate;
	/** Pre-allocated once to avoid reallocation while the render thread may be writing. */
	inline static TArray<FCollectedData> SegmentData;
	inline static int32 PrevActiveShotIndex;
	inline static bool bShowFlagsDisabled;
};
