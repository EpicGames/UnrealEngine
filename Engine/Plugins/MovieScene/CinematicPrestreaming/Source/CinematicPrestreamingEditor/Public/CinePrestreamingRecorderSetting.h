// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineOutputBase.h"
#include "MovieRenderPipelineDataTypes.h"
#include "CinePrestreamingRecorderSetting.generated.h"

class UCinePrestreamingData;
class UMovieScene;

USTRUCT(BlueprintType)
struct FMoviePipelineCinePrestreamingGeneratedData
{
	GENERATED_BODY()

	FMoviePipelineCinePrestreamingGeneratedData()
		: StreamingData(nullptr)
		, MovieScene(nullptr)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cinematic Prestreaming")
	TObjectPtr<UCinePrestreamingData> StreamingData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cinematic Prestreaming")
	TObjectPtr<UMovieScene> MovieScene;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cinematic Prestreaming")
	FString PackagePath;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cinematic Prestreaming")
	FString AssetName;

	TRange<FFrameNumber> Range;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnCinePrestreamingGenerateData, TArray<FMoviePipelineCinePrestreamingGeneratedData>, GeneratedData);

UCLASS(BlueprintType)
class CINEMATICPRESTREAMINGEDITOR_API UCinePrestreamingRecorderSetting : public UMoviePipelineOutputBase
{
	GENERATED_BODY()

public:
	UCinePrestreamingRecorderSetting();

protected:
	// UMoviePipelineSetting interface
	bool IsValidOnShots() const override { return true; }
	bool IsValidOnPrimary() const override { return true; }
#if WITH_EDITOR
	FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "CinePrestreamingSettingDisplayName", "Prestreaming Recorder"); }
	FText GetFooterText(UMoviePipelineExecutorJob* InJob) const;

#endif
	void SetupForPipelineImpl(UMoviePipeline* InPipeline) override;
	void TeardownForPipelineImpl(UMoviePipeline* InPipeline) override;
	// ~UMoviePipelineSetting Interface

private:
	void OnBeginFrame_GameThread();
	void OnEndFrame_GameThread();
	void OnEndFrame_RenderThread();
	void ModifyTargetSequences(const TArray<FMoviePipelineCinePrestreamingGeneratedData>& InData);
	void CreateAssetsFromData();

public:
	/* 
	* Specifies which directory the generated assets be placed in. Assumed to be relative to the root folder,
	* ie it should start with /Game/ etc. Can contain MRQ format strings such as {shot_name} which will
	* be resolved.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings",  meta=(ContentDir))
	FDirectoryPath PackageDirectory;
	
	/** Enable capture of virtual texture page requests. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite,Category = "Record")
	bool bVirtualTextures = true;
	/** Enable capture of nanite requests. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Record")
	bool bNanite = true;
	/** Automatically add the generated tracks to the target sequence */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Process")
	bool bModifyTargetSequence = true;
	/** Enable merging of frame data. This can reduce data overhead in the common case where there is high coherance of requests across frames. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations")
	bool bMergeFrames = false;
	/** Maximum number of frames to merge data across. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations", meta = (EditCondition = "bMergeFrames", UIMin = "1", ClampMin = "1"))
	int32 FrameCountMergeThreshold = 6;
	/** Threshold of virtual texture page requests. We merge frames until up to and including the frame where we pass this threshold. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations", meta = (EditCondition = "bVirtualTextures && bMergeFrames", UIMin = "1", ClampMin = "1"))
	int32 VirtualTextureRequestMergeThreshold = 2000;
	/** Threshold of nanite page requests. We merge frames until up to and including the frame where we pass this threshold. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations", meta = (EditCondition = "bNanite && bMergeFrames", UIMin = "1", ClampMin = "1"))
	int32 NaniteRequestMergeThreshold = 500;
	/** By default we disable any render features that aren't needed to generate the prestreaming data. This makes renders significantly faster, but also means that the final image generated by the renderer is unusable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optimizations")
	bool bDisableAdvanceRenderFeatures = true;
	/** First frame to output in each recorded asset. Frames before this are dropped. */
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin="0"))
	int32 StartFrame = 0;
	/** Last frame to capture in each recorded asset. Frames after this are dropped. (Set to 0 to ignore). */
	UPROPERTY(BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	int32 EndFrame = 0;
	
	UPROPERTY(BlueprintAssignable)
	FOnCinePrestreamingGenerateData OnGenerateData;

private:
	FDelegateHandle BeginFrameDelegate;
	FDelegateHandle EndFrameDelegate;

	struct FFrameData
	{
		TMap<uint64, uint32> VirtualTextureRequestData;
		TMap<uint64, uint32> NaniteRequestData;
	};

	struct FCollectedData
	{
		bool bValid = false;
		/** Map of time to recorded frame requests. */
		TMap<FFrameNumber, FFrameData> FrameData;
		/** OutputState required for creating output file names. */
		FMoviePipelineFrameOutputState OutputState;
		/** TickInRoot first tick in frame. */
		FFrameNumber InitialShotTick;
	};

	TArray<FCollectedData> SegmentData;
	int32 PrevActiveShotIndex;
	TArray<FString> ShowFlagsToDisable;
};
